// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshConversionEngineTypes : ModuleRules
{	
	public MeshConversionEngineTypes(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Note: Use the MeshConversion module for conversions of types that do not
		// require UObjects or Engine types, and this module for conversions that
		// do require Engine types.
		PublicDependencyModuleNames.AddRange(
			new string[] {
                "Core",
				"Chaos", // for geometry collection
				"Engine",
				"MeshDescription",
				"MeshConversion",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore"
            }
		);
    }
}
