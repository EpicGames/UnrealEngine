// Copyright Epic Games, Inc. All Rights Reserved. 

#include "DynamicMeshToSkeleton.h"

#include "Animation/Skeleton.h"

#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Logging/LogCategory.h"

DEFINE_LOG_CATEGORY_STATIC(LogDynamicMeshToSkeleton, Log, All);

namespace UE::Conversion
{
	using namespace UE::Geometry;

	namespace Private
	{
		bool DoesParentChainMatch(int32 StartBoneIndex, const FReferenceSkeleton& SkeletonRefSkel, const FDynamicMesh3& InDynamicMesh, const TMap<FName, int32>& BoneNameToID)
		{
			const int32* FoundMeshBoneIndex = BoneNameToID.Find(SkeletonRefSkel.GetBoneName(StartBoneIndex));
			if (!FoundMeshBoneIndex)
			{
				return false;
			}

			const FDynamicMeshBoneNameAttribute* BoneNames = InDynamicMesh.Attributes()->GetBoneNames();
			const FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = InDynamicMesh.Attributes()->GetBoneParentIndices();

			int32 SkeletonBoneIndex = StartBoneIndex;
			int32 MeshBoneIndex = *FoundMeshBoneIndex;

			do
			{
				const int32 ParentSkeletonIndex = SkeletonRefSkel.GetParentIndex(SkeletonBoneIndex);
				const int32 ParentMeshBoneIndex = BoneParentIndices->GetValue(MeshBoneIndex);

				if (ParentSkeletonIndex == INDEX_NONE || ParentMeshBoneIndex == INDEX_NONE)
				{
					return ParentSkeletonIndex == ParentMeshBoneIndex;
				}

				if (SkeletonRefSkel.GetBoneName(ParentSkeletonIndex) != BoneNames->GetValue(ParentMeshBoneIndex))
				{
					return false;
				}

				SkeletonBoneIndex = ParentSkeletonIndex;
				MeshBoneIndex = ParentMeshBoneIndex;
			} while (true);
		}
	}

	bool IsDynamicMeshCompatibleWithSkeleton(const FDynamicMesh3& InDynamicMesh, const USkeleton& Skeleton, bool bDoParentChainCheck)
	{
		if (!InDynamicMesh.HasAttributes() || !InDynamicMesh.Attributes()->HasBones())
		{
			return false;
		}

		const FReferenceSkeleton& SkeletonRefSkel = Skeleton.GetReferenceSkeleton();
		if (SkeletonRefSkel.GetRawBoneNum() == 0)
		{
			return false;
		}

		const FDynamicMeshBoneNameAttribute* BoneNames = InDynamicMesh.Attributes()->GetBoneNames();
		const FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = InDynamicMesh.Attributes()->GetBoneParentIndices();
		const int32 NumBones = InDynamicMesh.Attributes()->GetNumBones();

		TMap<FName, int32> BoneNameToIndex;
		BoneNameToIndex.Reserve(NumBones);
		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			BoneNameToIndex.Add(BoneNames->GetValue(BoneIdx), BoneIdx);
		}

		int32 NumOfBoneMatches = 0;

		for (int32 MeshBoneIndex = 0; MeshBoneIndex < NumBones; ++MeshBoneIndex)
		{
			const FName MeshBoneName = BoneNames->GetValue(MeshBoneIndex);
			int32 SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex(MeshBoneName);
			if (SkeletonBoneIndex != INDEX_NONE)
			{
				++NumOfBoneMatches;

				if (bDoParentChainCheck && !Private::DoesParentChainMatch(SkeletonBoneIndex, SkeletonRefSkel, InDynamicMesh, BoneNameToIndex))
				{
					UE_LOGF(LogDynamicMeshToSkeleton, Verbose, "%ls : Hierarchy does not match.", *MeshBoneName.ToString());
					return false;
				}
			}
			else
			{
				int32 CurrentMeshBoneIndex = MeshBoneIndex;
				while (SkeletonBoneIndex == INDEX_NONE)
				{
					CurrentMeshBoneIndex = BoneParentIndices->GetValue(CurrentMeshBoneIndex);
					if (CurrentMeshBoneIndex == INDEX_NONE)
					{
						break;
					}
					SkeletonBoneIndex = SkeletonRefSkel.FindBoneIndex(BoneNames->GetValue(CurrentMeshBoneIndex));
				}

				if (SkeletonBoneIndex == INDEX_NONE)
				{
					UE_LOGF(LogDynamicMeshToSkeleton, Verbose, "%ls : Missing joint on skeleton.  Make sure to assign to the skeleton.", *MeshBoneName.ToString());
					return false;
				}

				if (bDoParentChainCheck && !Private::DoesParentChainMatch(SkeletonBoneIndex, SkeletonRefSkel, InDynamicMesh, BoneNameToIndex))
				{
					UE_LOGF(LogDynamicMeshToSkeleton, Verbose, "%ls : Hierarchy does not match.", *MeshBoneName.ToString());
					return false;
				}
			}
		}

		return NumOfBoneMatches > 0;
	}
}