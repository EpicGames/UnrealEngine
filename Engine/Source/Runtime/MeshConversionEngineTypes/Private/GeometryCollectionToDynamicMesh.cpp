// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionToDynamicMesh.h"

#include "Containers/ArrayView.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshEditor.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"

#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionUVFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"

namespace UE::Geometry
{

TConstArrayView<FTransform3f> FGeometryCollectionToDynamicMeshes::GetCollectionTransforms(const FManagedArrayCollection& Collection)
{
	const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
	if (TransformFacade.IsValid())
	{
		if (const TManagedArray<FTransform3f>* Transforms = TransformFacade.FindTransforms())
		{
			return TConstArrayView<FTransform3f>(Transforms->GetConstArray());
		}
	}
	return TConstArrayView<FTransform3f>();
}

bool FGeometryCollectionToDynamicMeshes::InitHelper(const FManagedArrayCollection& Collection, bool bTransformInComponentSpace, TConstArrayView<FTransform3f> Transforms, bool bAllTransforms, TConstArrayView<int32> TransformIndices, const FToMeshOptions& Options)
{
	Meshes.Reset();

	const GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(Collection);
	if (!MeshFacade.IsValid())
	{
		return false;
	}
	TManagedArrayAccessor<int32> ParentAttribute(Collection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
	if (!ParentAttribute.IsValid())
	{
		return false;
	}
	TManagedArrayAccessor<int32> SimTypeAttribute(Collection, FGeometryCollection::SimulationTypeAttribute, FTransformCollection::TransformGroup);

	if (Transforms.IsEmpty())
	{
		return true; // successfully converted an empty collection?
	}

	const GeometryCollection::UV::FConstUVLayers UVLayers = GeometryCollection::UV::FindActiveUVLayers(Collection);

	TConstArrayView<int32> UseTransformIndices = TransformIndices;
	TArray<int32> LocalTransformIndices;
	if (bAllTransforms)
	{
		LocalTransformIndices.SetNum(Transforms.Num());
		for (int32 Idx = 0; Idx < Transforms.Num(); ++Idx)
		{
			LocalTransformIndices[Idx] = Idx;
		}
		UseTransformIndices = TConstArrayView<int32>(LocalTransformIndices);
	}

	for (int32 TransformIdx : UseTransformIndices)
	{
		int32 GeometryIdx = MeshFacade.TransformToGeometryIndexAttribute[TransformIdx];
		if (GeometryIdx == INDEX_NONE)
		{
			// currently only converting transforms with associated geometry
			continue;
		}
		if (SimTypeAttribute.IsValid() && SimTypeAttribute[TransformIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			// if we have sim type info, also restrict to rigids -- cluster geometry is typically just there for legacy reasons
			// (TODO: expose as option)
			continue;
		}

		FTransformSRT3d CollectionToLocal;
		if (bTransformInComponentSpace)
		{
			CollectionToLocal = FTransformSRT3d(FTransform(Transforms[TransformIdx]) * Options.Transform);
		}
		else
		{
			CollectionToLocal = FTransformSRT3d(GeometryCollectionAlgo::GlobalMatrix(Transforms, TArrayView<const int32>(ParentAttribute.Get().GetConstArray()), TransformIdx) * Options.Transform);
		}

		int32 AddedMeshIdx = Meshes.Emplace();
		FMeshInfo& MeshData = Meshes[AddedMeshIdx];
		MeshData.Transform = FTransform(CollectionToLocal);
		MeshData.TransformIndex = TransformIdx;
		MeshData.Mesh = MakeUnique<FDynamicMesh3>();
		FDynamicMesh3& Mesh = *MeshData.Mesh;

		Mesh.EnableAttributes();
		Mesh.Attributes()->EnableMaterialID();
		Mesh.Attributes()->EnablePrimaryColors();
		Mesh.Attributes()->EnableTangents();
		Mesh.Attributes()->SetNumUVLayers(UVLayers.Num());
		bool bInvisiblePolygroup = Options.InvisibleFaces != EInvisibleFaceConversion::TagWithPolygroup;
		int32 NumCustomPolygroupLayers = (int32)Options.bInternalFaceTagsAsPolygroups + (int32)bInvisiblePolygroup;
		FDynamicMeshPolygroupAttribute* InternalFacePolygroups = nullptr;
		FDynamicMeshPolygroupAttribute* VisibleFacePolygroups = nullptr;
		Mesh.Attributes()->SetNumPolygroupLayers(NumCustomPolygroupLayers);
		{
			int32 CustomLayerIdx = 0;
			if (Options.bInternalFaceTagsAsPolygroups)
			{
				InternalFacePolygroups = Mesh.Attributes()->GetPolygroupLayer(CustomLayerIdx);
				InternalFacePolygroups->SetName(InternalFacePolyGroupName());
				CustomLayerIdx++;
			}
			if (bInvisiblePolygroup)
			{
				VisibleFacePolygroups = Mesh.Attributes()->GetPolygroupLayer(CustomLayerIdx);
				VisibleFacePolygroups->SetName(VisibleFacePolyGroupName());
				CustomLayerIdx++;
			}
		}

		const int32 VertexStart = MeshFacade.VertexStartAttribute[GeometryIdx];
		const int32 VertexCount = MeshFacade.VertexCountAttribute[GeometryIdx];
		const TArray<FVector3f>& VertexPositions = MeshFacade.VertexAttribute.Get().GetConstArray();
		const TArray<FLinearColor>& VertexColors = MeshFacade.ColorAttribute.Get().GetConstArray();
		const TArray<FVector3f>& VertexNormals = MeshFacade.NormalAttribute.Get().GetConstArray();
		const TArray<FVector3f>& VertexTangents = MeshFacade.NormalAttribute.Get().GetConstArray();
		const TArray<FVector3f>& VertexBitangents = MeshFacade.NormalAttribute.Get().GetConstArray();

		for (int32 Idx = VertexStart, N = VertexStart + VertexCount; Idx < N; ++Idx)
		{
			FVector3d Position = CollectionToLocal.TransformPosition(FVector3d(VertexPositions[Idx]));
			int32 VID = Mesh.AppendVertex(Position);
			// add overlay elements -- by construction, will be 1:1 with vertices initially
			const int32 ColorEID = Mesh.Attributes()->PrimaryColors()->AppendElement((FVector4f)VertexColors[Idx]);
			checkSlow(ColorEID == VID);
			const int32 NormalEID = Mesh.Attributes()->PrimaryNormals()->AppendElement(VertexNormals[Idx]);
			checkSlow(NormalEID == VID);
			const int32 TangentEID = Mesh.Attributes()->PrimaryTangents()->AppendElement(VertexTangents[Idx]);
			checkSlow(TangentEID == VID);
			const int32 BiTangentEID = Mesh.Attributes()->PrimaryBiTangents()->AppendElement(VertexBitangents[Idx]);
			checkSlow(BiTangentEID == VID);
			for (int32 UVLayer = 0; UVLayer < UVLayers.Num(); ++UVLayer)
			{
				const int32 UVElID = Mesh.Attributes()->GetUVLayer(UVLayer)->AppendElement(UVLayers[UVLayer][Idx]);
				checkSlow(UVElID == VID);
			}
		}

		// TODO: optionally enable non-manifold vertex mapping?

		FIntVector VertexOffset(VertexStart, VertexStart, VertexStart);
		const int32 FaceStart = MeshFacade.FaceStartAttribute[GeometryIdx];
		const int32 FaceCount = MeshFacade.FaceCountAttribute[GeometryIdx];

		const TArray<FIntVector>& FacesArray = MeshFacade.IndicesAttribute.Get().GetConstArray();
		const TBitArray<>& VisibleArray = MeshFacade.VisibleAttribute.Get().GetConstArray();
		const TBitArray<>& InternalArray = MeshFacade.InternalAttribute.Get().GetConstArray();
		const TArray<int32>& MaterialIDs = MeshFacade.MaterialIDAttribute.Get().GetConstArray();

		bool bSrcIsManifold = true;

		for (int32 FaceIdx = FaceStart, N = FaceStart + FaceCount; FaceIdx < N; ++FaceIdx)
		{
			if (Options.InvisibleFaces == EInvisibleFaceConversion::Skip && !VisibleArray[FaceIdx])
			{
				continue;
			}
			FIndex3i AddTri = FacesArray[FaceIdx] - VertexOffset;
			int32 TID = Mesh.AppendTriangle(AddTri, 0);
			if (TID == FDynamicMesh3::NonManifoldID)
			{
				bSrcIsManifold = false;

				int E0 = Mesh.FindEdge(AddTri[0], AddTri[1]);
				int E1 = Mesh.FindEdge(AddTri[1], AddTri[2]);
				int E2 = Mesh.FindEdge(AddTri[2], AddTri[0]);

				// determine which verts need to be duplicated
				bool bDuplicate[3] = { false, false, false };
				if (E0 != FDynamicMesh3::InvalidID && Mesh.IsBoundaryEdge(E0) == false)
				{
					bDuplicate[0] = true;
					bDuplicate[1] = true;
				}
				if (E1 != FDynamicMesh3::InvalidID && Mesh.IsBoundaryEdge(E1) == false)
				{
					bDuplicate[1] = true;
					bDuplicate[2] = true;
				}
				if (E2 != FDynamicMesh3::InvalidID && Mesh.IsBoundaryEdge(E2) == false)
				{
					bDuplicate[2] = true;
					bDuplicate[0] = true;
				}
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					if (bDuplicate[SubIdx])
					{
						// duplicate vertex and all overlay elements
						int32 ToDupVID = AddTri[SubIdx];
						FVector3d Position = Mesh.GetVertex(ToDupVID);
						int32 NewVID = Mesh.AppendVertex(Position);
						int32 ColorEID = Mesh.Attributes()->PrimaryColors()->AppendElement(Mesh.Attributes()->PrimaryColors()->GetElement(ToDupVID));
						checkSlow(ColorEID == NewVID);
						for (int32 NormalLayerIdx = 0; NormalLayerIdx < 3; ++NormalLayerIdx)
						{
							const int32 NormalEID = Mesh.Attributes()->GetNormalLayer(NormalLayerIdx)->AppendElement(Mesh.Attributes()->GetNormalLayer(NormalLayerIdx)->GetElement(ToDupVID));
							checkSlow(NormalEID == NewVID);
						}
						for (int32 UVLayer = 0; UVLayer < UVLayers.Num(); ++UVLayer)
						{
							const int32 UVElID = Mesh.Attributes()->GetUVLayer(UVLayer)->AppendElement(Mesh.Attributes()->GetUVLayer(UVLayer)->GetElement(ToDupVID));
							checkSlow(UVElID == NewVID);
						}
						// set new vertex ID on triangle
						AddTri[SubIdx] = NewVID;
					}
				}

				// try to append again
				TID = Mesh.AppendTriangle(AddTri);
				checkSlow(TID != FDynamicMesh3::NonManifoldID);
			}
			if (TID < 0)
			{
				continue;
			}
			// after successfully appending the triangle, set overlay elements to match (since overlay elements are 1:1 with vertices in initial mesh)
			Mesh.Attributes()->PrimaryColors()->SetTriangle(TID, AddTri, false);
			for (int32 NormalLayerIdx = 0; NormalLayerIdx < 3; ++NormalLayerIdx)
			{
				Mesh.Attributes()->GetNormalLayer(NormalLayerIdx)->SetTriangle(TID, AddTri, false);
			}
			for (int32 UVLayerIdx = 0; UVLayerIdx < UVLayers.Num(); ++UVLayerIdx)
			{
				Mesh.Attributes()->GetUVLayer(UVLayerIdx)->SetTriangle(TID, AddTri, false);
			}

			Mesh.Attributes()->GetMaterialID()->SetValue(TID, MaterialIDs[FaceIdx]);
			if (InternalFacePolygroups)
			{
				InternalFacePolygroups->SetValue(TID, 1 + (int32)InternalArray[FaceIdx]);
			}
			if (VisibleFacePolygroups)
			{
				VisibleFacePolygroups->SetValue(TID, 1 + (int32)VisibleArray[FaceIdx]);
			}
			// note: material index doesn't need to be passed through; will be rebuilt by a call to reindex materials once the cut mesh is returned back to geometry collection format
		}

		if (!Options.bSaveIsolatedVertices)
		{
			FDynamicMeshEditor Editor(&Mesh);
			Editor.RemoveIsolatedVertices();
		}

		if (Options.bWeldVertices)
		{
			FMergeCoincidentMeshEdges Welder(&Mesh);
			Welder.MergeVertexTolerance = FMathd::Epsilon;
			Welder.bWeldAttrsOnMergedEdges = true;
			Welder.Apply();
		}

		// if isolated verts were removed or edges were welded, may benefit from compacting the mesh
		Mesh.CompactInPlace();
	}
	
	return true;
}

// Convert all geometry in the given Collection to dynamic meshes
bool FGeometryCollectionToDynamicMeshes::Init(const FManagedArrayCollection& Collection, const FToMeshOptions& Options)
{
	return InitHelper(Collection, false, GetCollectionTransforms(Collection), true, TConstArrayView<int32>(), Options);
}
// Convert geometry at selected transform indices in the given Collection to dynamic meshes
bool FGeometryCollectionToDynamicMeshes::InitFromTransformSelection(const FManagedArrayCollection& Collection, TConstArrayView<int32> TransformIndices, const FToMeshOptions& Options)
{
	return InitHelper(Collection, false, GetCollectionTransforms(Collection), false, TransformIndices, Options);
}

namespace
{
	// Track instances of parent vertices with unique element IDs
	template<int MaxUVLayers = 8>
	struct TUniqueMeshVertex
	{
		// Note: We don't include the parent VID here, so verts with fully unset attribs will not be distinguishable (e.g., elements w/ no triangle)

		// Element IDs for UV overlays
		TStaticArray<int32, MaxUVLayers> UVsEIDs;
		// Element IDs for Normal, Tangent and BiTangent overlays
		TStaticArray<int32, 3> NormalEIDs;
		// Element ID for color overlay
		int32 ColorEID;

		TUniqueMeshVertex()
		{
			InitEmpty();
		}
		TUniqueMeshVertex(const FDynamicMesh3& Mesh, int32 TID, int32 SubIdx)
		{
			if (!Mesh.HasAttributes())
			{
				InitEmpty();
				return;
			}
			const int32 NumNormalLayers = FMath::Min(3, Mesh.Attributes()->NumNormalLayers());
			for (int32 NormalLayerIdx = 0; NormalLayerIdx < NumNormalLayers; ++NormalLayerIdx)
			{
				NormalEIDs[NormalLayerIdx] = Mesh.Attributes()->GetNormalLayer(NormalLayerIdx)->GetTriangle(TID)[SubIdx];
			}
			for (int32 NormalLayerIdx = NumNormalLayers; NormalLayerIdx < 3; ++NormalLayerIdx)
			{
				NormalEIDs[NormalLayerIdx] = -1;
			}
			ColorEID = Mesh.Attributes()->PrimaryColors() ? Mesh.Attributes()->PrimaryColors()->GetTriangle(TID)[SubIdx] : -1;
			int32 NumUVs = FMath::Min(MaxUVLayers, Mesh.Attributes()->NumUVLayers()); // max 8 UV layers supported
			for (int32 UVIdx = 0; UVIdx < NumUVs; ++UVIdx)
			{
				UVsEIDs[UVIdx] = (Mesh.Attributes()->GetUVLayer(UVIdx)->GetTriangle(TID)[SubIdx]);
			}
			for (int32 UVIdx = NumUVs; UVIdx < MaxUVLayers; ++UVIdx)
			{
				UVsEIDs[UVIdx] = INDEX_NONE;
			}
		}

		inline void InitEmpty()
		{
			ColorEID = -1;
			NormalEIDs = { -1,-1,-1 };
			for (int32 UVIdx = 0; UVIdx < MaxUVLayers; ++UVIdx)
			{
				UVsEIDs[UVIdx] = -1;
			}
		}

		bool operator==(const TUniqueMeshVertex& Other) const
		{

			bool bEquality = true;
			for (int32 UVLayerIdx = 0; UVLayerIdx < MaxUVLayers; ++UVLayerIdx)
			{
				bEquality &= (this->UVsEIDs[UVLayerIdx] == Other.UVsEIDs[UVLayerIdx]);
			}
			bEquality &= (this->NormalEIDs[0] == Other.NormalEIDs[0]);
			bEquality &= (this->NormalEIDs[1] == Other.NormalEIDs[1]);
			bEquality &= (this->NormalEIDs[2] == Other.NormalEIDs[2]);
			bEquality &= (this->ColorEID == Other.ColorEID);


			return bEquality;
		}
	};

}
// Convert currently-held meshes back to geometry in the Collection, overwriting existing geometry as needed
bool FGeometryCollectionToDynamicMeshes::UpdateGeometryCollection(FGeometryCollection& Collection, const FToCollectionOptions& Options) const
{
	bool bAllSucceeded = true;

	int32 NumGeometry = Collection.NumElements(FGeometryCollection::GeometryGroup);
	TArray<int32> NewFaceCounts, NewVertexCounts;
	NewFaceCounts.SetNumUninitialized(NumGeometry);
	NewVertexCounts.SetNumUninitialized(NumGeometry);
	for (int32 GeomIdx = 0; GeomIdx < Collection.FaceCount.Num(); GeomIdx++)
	{
		NewFaceCounts[GeomIdx] = Collection.FaceCount[GeomIdx];
		NewVertexCounts[GeomIdx] = Collection.VertexCount[GeomIdx];
	}
	bool bNeedsSplits = false, bNeedsResize = false;
	
	TArray<TUniqueMeshVertex<>> Uniques;
	auto CountUniqueVerts = [&Uniques](const FDynamicMesh3& Mesh)
	{
		int32 Count = 0;
		for (int32 VID : Mesh.VertexIndicesItr())
		{
			Uniques.Reset();
			for (int32 TID : Mesh.VtxTrianglesItr(VID))
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				int32 SubIdx = Tri.IndexOf(VID);
				Uniques.AddUnique(TUniqueMeshVertex<>(Mesh, TID, SubIdx));
			}
			Count += FMath::Max(1, Uniques.Num());
		}
		return Count;
	};
	int32 NeedsAppend = 0;
	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		const FMeshInfo& MeshData = Meshes[MeshIdx];
		if (MeshData.Mesh.IsValid())
		{
			if (Collection.TransformToGeometryIndex.IsValidIndex(MeshData.TransformIndex))
			{
				int32 GeomIdx = Collection.TransformToGeometryIndex[MeshData.TransformIndex];
				int32 PrevTriCount = NewFaceCounts[GeomIdx];
				NewFaceCounts[GeomIdx] = MeshData.Mesh->TriangleCount();
				bNeedsResize = bNeedsResize || NewFaceCounts[GeomIdx] != PrevTriCount;
				int32 PrevVertexCount = NewVertexCounts[GeomIdx];
				NewVertexCounts[GeomIdx] = CountUniqueVerts(*MeshData.Mesh);
				bNeedsSplits = bNeedsSplits || NewVertexCounts[GeomIdx] != MeshData.Mesh->VertexCount();
				bNeedsResize = bNeedsResize || NewVertexCounts[GeomIdx] != PrevVertexCount;
			}
			else
			{
				++NeedsAppend;
			}
		}
	}

	if (bNeedsResize)
	{
		bool bDoValidation = false;
#if UE_BUILD_DEBUG
		bDoValidation = true; // note: this validation is extremely slow
#endif
		GeometryCollectionAlgo::ResizeGeometries(&Collection, NewFaceCounts, NewVertexCounts, bDoValidation);
	}

	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		const FMeshInfo& MeshData = Meshes[MeshIdx];
		if (!MeshData.Mesh.IsValid()
			|| !Collection.TransformToGeometryIndex.IsValidIndex(MeshData.TransformIndex))
		{
			continue;
		}
		const FDynamicMesh3& Mesh = *MeshData.Mesh;
		int32 GeometryIdx = Collection.TransformToGeometryIndex[MeshData.TransformIndex];
		bool bSucceeded = UpdateCollection(MeshData.Transform, Mesh, GeometryIdx, Collection, Options);
		bAllSucceeded &= bSucceeded;
	}
	if (NeedsAppend > 0)
	{
		for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
		{
			const FMeshInfo& MeshData = Meshes[MeshIdx];
			if (!MeshData.Mesh.IsValid()
				|| Collection.TransformToGeometryIndex.IsValidIndex(MeshData.TransformIndex))
			{
				continue;
			}
			const FDynamicMesh3& Mesh = *MeshData.Mesh;
			const int32 NewTransformIdx = AppendMeshToCollection(Collection, Mesh, MeshData.Transform, Options);

			bool bSucceeded = NewTransformIdx != INDEX_NONE;
			bAllSucceeded &= bSucceeded;
		}
	}

	return bAllSucceeded;
}

int32 FGeometryCollectionToDynamicMeshes::AppendMeshToCollection(FGeometryCollection& Collection, const FDynamicMesh3& Mesh, const FTransform& MeshTransform, const FToCollectionOptions& Options)
{
	TArray<TUniqueMeshVertex<>> Uniques;
	auto CountUniqueVerts = [&Uniques](const FDynamicMesh3& Mesh)
	{
		int32 Count = 0;
		for (int32 VID : Mesh.VertexIndicesItr())
		{
			Uniques.Reset();
			for (int32 TID : Mesh.VtxTrianglesItr(VID))
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				int32 SubIdx = Tri.IndexOf(VID);
				Uniques.AddUnique(TUniqueMeshVertex<>(Mesh, TID, SubIdx));
			}
			Count += FMath::Max(1, Uniques.Num());
		}
		return Count;
	};

	const int32 GeometryIdx = Collection.AddElements(1, FGeometryCollection::GeometryGroup);
	const int32 TransformIdx = Collection.AddElements(1, FGeometryCollection::TransformGroup);
	const int32 NumTriangles = Mesh.TriangleCount();
	const int32 NumVertices = CountUniqueVerts(Mesh);
	const int32 FacesStart = Collection.AddElements(NumTriangles, FGeometryCollection::FacesGroup);
	const int32 VerticesStart = Collection.AddElements(NumVertices, FGeometryCollection::VerticesGroup);

	Collection.FaceCount[GeometryIdx] = NumTriangles;
	Collection.FaceStart[GeometryIdx] = FacesStart;
	Collection.VertexCount[GeometryIdx] = NumVertices;
	Collection.VertexStart[GeometryIdx] = VerticesStart;
	Collection.TransformIndex[GeometryIdx] = TransformIdx;
	Collection.TransformToGeometryIndex[TransformIdx] = GeometryIdx;
	int32 TransformParent = Options.NewMeshParentIndex;
	// Don't allow the requested parent to be the new transform itself, or a transform that doesn't exist yet
	if (TransformParent >= TransformIdx)
	{
		TransformParent = INDEX_NONE;
	}
	// If parent is INDEX_NONE, and we're not allowing geometry to be appended directly as root, search for a cluster root (other than self) to use as parent
	if (!Options.bAllowAppendAsRoot)
	{
		if (TransformParent == INDEX_NONE)
		{
			for (int32 Idx = 0; Idx < TransformIdx; ++Idx)
			{
				if (Collection.Parent[Idx] == INDEX_NONE && Collection.SimulationType[Idx] == FGeometryCollection::ESimulationTypes::FST_Clustered)
				{
					TransformParent = Idx;
				}
			}
		}
		// If still haven't found a valid parent, add a new root as parent (can occur if collection was empty)
		if (!Collection.Parent.IsValidIndex(TransformParent))
		{
			TransformParent = Collection.AddElements(1, FGeometryCollection::TransformGroup);
			Collection.Parent[TransformParent] = INDEX_NONE;
			Collection.BoneColor[TransformParent] = FLinearColor::White;
		}
	}

	if (TransformParent > INDEX_NONE)
	{
		Collection.BoneName[TransformIdx] = Collection.BoneName[TransformParent] + "_" + FString::FromInt(Collection.Children[TransformParent].Num());
		Collection.BoneColor[TransformIdx] = Collection.BoneColor[TransformParent];
		Collection.Children[TransformParent].Add(TransformIdx);
		Collection.SimulationType[TransformParent] = FGeometryCollection::ESimulationTypes::FST_Clustered;
	}
	else
	{
		Collection.BoneName[TransformIdx] = FString::FromInt(Collection.BoneName.Num());
		Collection.BoneColor[TransformIdx] = FLinearColor::White;
	}
	Collection.Parent[TransformIdx] = TransformParent;

	Collection.Transform[TransformIdx] = FTransform3f::Identity;
	Collection.SimulationType[TransformIdx] = FGeometryCollection::ESimulationTypes::FST_Rigid;

	bool bSuccess = UpdateCollection(MeshTransform, Mesh, GeometryIdx, Collection, Options);
	if (!ensure(bSuccess)) // Note: we should have set up the geometry collection above such that UpdateCollection won't fail
	{
		return INDEX_NONE;
	}
	return TransformIdx;
}


// Update an existing geometry in a collection w/ a new mesh (w/ the same number of faces and vertices!)
bool FGeometryCollectionToDynamicMeshes::UpdateCollection(const FTransform& FromCollection, const FDynamicMesh3& Mesh, int32 GeometryIdx, FGeometryCollection& Output, const FToCollectionOptions& Options)
{
	int32 VertexCount = Output.VertexCount[GeometryIdx];
	int32 VertexStart = Output.VertexStart[GeometryIdx];
	int32 TriangleCount = Output.FaceCount[GeometryIdx];
	int32 TriangleStart = Output.FaceStart[GeometryIdx];

	int32 UVLayerCount = Mesh.HasAttributes() ? Mesh.Attributes()->NumUVLayers() : 1;
	Output.SetNumUVLayers(UVLayerCount);
	GeometryCollection::UV::FUVLayers OutputUVLayers = GeometryCollection::UV::FindActiveUVLayers(Output);

	int32 TransformIdx = Output.TransformIndex[GeometryIdx];

	FBox Bounds;

	// map from mesh vertex IDs to geometry collection indices
	TArray<int32> VertexIDToIdxStart;
	VertexIDToIdxStart.Init(INDEX_NONE, Mesh.MaxVertexID());
	TArray<TUniqueMeshVertex<>> PerIdxElements;
	PerIdxElements.Reserve(VertexCount);
	// Temp storage for unique elements on a given vertex
	TArray<TUniqueMeshVertex<>> Uniques;
	{
		int32 Count = 0;
		for (int32 VID : Mesh.VertexIndicesItr())
		{
			VertexIDToIdxStart[VID] = PerIdxElements.Num();
			Uniques.Reset();
			FVector3f Pos = (FVector3f)FromCollection.InverseTransformPosition(FVector(Mesh.GetVertex(VID)));
			Bounds += (FVector)Pos;
			for (int32 TID : Mesh.VtxTrianglesItr(VID))
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				int32 SubIdx = Tri.IndexOf(VID);
				Uniques.AddUnique(TUniqueMeshVertex<>(Mesh, TID, SubIdx));
			}
			if (Uniques.IsEmpty())
			{
				Uniques.Emplace();
			}
			for (int32 SubIdx = 0; SubIdx < Uniques.Num(); ++SubIdx)
			{
				int32 CopyToIdx = VertexStart + PerIdxElements.Add(Uniques[SubIdx]);
				Output.Vertex[CopyToIdx] = Pos;
				Output.BoneMap[CopyToIdx] = TransformIdx;
				FVector3f NormalVals[3]{ FVector3f::ZAxisVector, FVector3f::XAxisVector, FVector3f::YAxisVector };
				if (Mesh.HasAttributes())
				{
					int32 NumNormalLayers = FMath::Min(3, Mesh.Attributes()->NumNormalLayers());
					for (int32 NormalLayerIdx = 0; NormalLayerIdx < NumNormalLayers; ++NormalLayerIdx)
					{
						int32 ElID = Uniques[SubIdx].NormalEIDs[NormalLayerIdx];
						if (ElID > -1)
						{
							NormalVals[NormalLayerIdx] = Mesh.Attributes()->GetNormalLayer(NormalLayerIdx)->GetElement(ElID);
						}
					}
				}
				Output.Normal[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(NormalVals[0]));
				Output.TangentU[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(NormalVals[1]));
				Output.TangentV[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(NormalVals[2]));
				if (const FDynamicMeshColorOverlay* Colors = Mesh.Attributes()->PrimaryColors())
				{
					int32 ElID = Uniques[SubIdx].ColorEID;
					if (ElID > -1)
					{
						Output.Color[CopyToIdx] = FLinearColor(Colors->GetElement(ElID));
					}
				}

				for (int32 UVLayer = 0; UVLayer < UVLayerCount; ++UVLayer)
				{
					FVector2f UV(0, 0);
					if (const FDynamicMeshUVOverlay* UVs = Mesh.Attributes()->GetUVLayer(UVLayer))
					{
						int32 ElID = Uniques[SubIdx].UVsEIDs[UVLayer];
						if (ElID > -1)
						{
							UV = UVs->GetElement(ElID);
						}
					}
					OutputUVLayers[UVLayer][CopyToIdx] = UV;
				}
			}
		}


	}

	check(PerIdxElements.Num() == VertexCount);

	const FDynamicMeshPolygroupAttribute* VisLayer = nullptr;
	const FDynamicMeshPolygroupAttribute* InternalLayer = nullptr;

	if (Mesh.HasAttributes())
	{
		for (int32 Idx = 0; Idx < Mesh.Attributes()->NumPolygroupLayers(); ++Idx)
		{
			const FDynamicMeshPolygroupAttribute* Layer = Mesh.Attributes()->GetPolygroupLayer(Idx);
			if (Layer->GetName() == VisibleFacePolyGroupName())
			{
				VisLayer = Layer;
			}
			if (Layer->GetName() == InternalFacePolyGroupName())
			{
				InternalLayer = Layer;
			}
		}
	}

	FIntVector VertexStartOffset(VertexStart);
	int32 TriIdxOffset = 0;
	for (int32 TID : Mesh.TriangleIndicesItr())
	{
		int32 CopyToIdx = TriangleStart + TriIdxOffset;
		TriIdxOffset++;
		bool bVis = Options.bDefaultFaceVisible, bInternal = Options.bDefaultFaceInternal;
		int32 MatID = 0;
		if (Mesh.HasAttributes())
		{
			if (VisLayer)
			{
				int32 VisTag = VisLayer->GetValue(TID) - 1;
				if (VisTag > -1)
				{
					bVis = bool(VisTag);
				}
			}
			if (InternalLayer)
			{
				int32 InternalTag = InternalLayer->GetValue(TID) - 1;
				if (InternalTag > -1)
				{
					bInternal = bool(InternalTag);
				}	
			}
			if (const FDynamicMeshMaterialAttribute* Mats = Mesh.Attributes()->GetMaterialID())
			{
				MatID = Mats->GetValue(TID);
			}
		}

		Output.Visible[CopyToIdx] = bVis;
		Output.Internal[CopyToIdx] = bInternal;
		Output.MaterialID[CopyToIdx] = MatID;
		FIndex3i Tri = Mesh.GetTriangle(TID);
		FIntVector OutTri;
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			TUniqueMeshVertex<> Unique(Mesh, TID, SubIdx);
			int32 MeshVID = Tri[SubIdx];
			int32 StartIdx = VertexIDToIdxStart[MeshVID];
			int32 NextMeshVID = MeshVID + 1;
			// Search for a valid NextMeshVID (in case mesh is not compact)
			while (NextMeshVID < VertexIDToIdxStart.Num() && VertexIDToIdxStart[NextMeshVID] == INDEX_NONE)
			{
				NextMeshVID++;
			}
			int32 EndIdx = NextMeshVID < VertexIDToIdxStart.Num() ? VertexIDToIdxStart[NextMeshVID] : PerIdxElements.Num();
			int32 FoundIdx = -1;
			for (int32 TestIdx = StartIdx; TestIdx < EndIdx; ++TestIdx)
			{
				if (PerIdxElements[TestIdx] == Unique)
				{
					FoundIdx = TestIdx;
					break;
				}
			}
			if (ensure(FoundIdx > -1))
			{
				OutTri[SubIdx] = FoundIdx + VertexStart;
			}
			else // (should not happen)
			{
				OutTri[SubIdx] = StartIdx + VertexStart;
			}
		}
		Output.Indices[CopyToIdx] = OutTri;
	}
	ensure(TriIdxOffset == TriangleCount);
	if (Output.BoundingBox.Num())
	{
		Output.BoundingBox[GeometryIdx] = Bounds;
	}

	return true;
}


FName FGeometryCollectionToDynamicMeshes::InternalFacePolyGroupName()
{
	return FName("GeometryCollectionInternalFaces");
}
FName FGeometryCollectionToDynamicMeshes::VisibleFacePolyGroupName()
{
	return FName("GeometryCollectionVisibleFaces");
}

} // namespace UE::Geometry