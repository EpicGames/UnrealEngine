// Copyright Epic Games, Inc. All Rights Reserved. 

#include "IndexedMeshToDynamicMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"

using namespace UE::Geometry;

bool UE::Conversion::RenderBuffersToDynamicMesh(
	const FStaticMeshVertexBuffers& VertexData,
	const FRawStaticIndexBuffer& IndexData,
	const FStaticMeshSectionArray& SectionData,
	FDynamicMesh3& MeshOut,
	bool bAttemptToWeldSeams)
{
	if (!ensureMsgf(VertexData.StaticMeshVertexBuffer.GetAllowCPUAccess(), TEXT("bAllowCPUAccess must be set to true for StaticMeshes before calling RenderBuffersToDynamicMesh(), otherwise the mesh geometry data isn't accessible!")))
	{
		return false;
	}
	const FStaticMeshVertexBuffer& VertexBuffer = VertexData.StaticMeshVertexBuffer;
	const FPositionVertexBuffer& PositionBuffer = VertexData.PositionVertexBuffer;
	const FColorVertexBuffer& ColorBuffer = VertexData.ColorVertexBuffer;

	int32 NumUVChannels = VertexBuffer.GetNumTexCoords();
	bool bHasUVs = (NumUVChannels > 0);
	NumUVChannels = FMath::Max(1, NumUVChannels);

	MeshOut.Clear();

	MeshOut.EnableAttributes();
	FDynamicMeshNormalOverlay* Normals = MeshOut.Attributes()->PrimaryNormals();
	MeshOut.Attributes()->EnableTangents();
	FDynamicMeshNormalOverlay* Tangents = MeshOut.Attributes()->PrimaryTangents();
	FDynamicMeshNormalOverlay* BiTangents = MeshOut.Attributes()->PrimaryBiTangents();

	MeshOut.Attributes()->SetNumUVLayers(NumUVChannels);
	TArray<FDynamicMeshUVOverlay*> UVs;
	for (int32 ui = 0; ui < NumUVChannels; ++ui)
	{
		UVs.Add(MeshOut.Attributes()->GetUVLayer(ui));
	}
	MeshOut.Attributes()->EnableMaterialID();
	FDynamicMeshMaterialAttribute* MaterialID = MeshOut.Attributes()->GetMaterialID();

	int32 NumVertices = PositionBuffer.GetNumVertices();
	for (int32 vi = 0; vi < NumVertices; ++vi)
	{
		FVector3f Position = PositionBuffer.VertexPosition(vi);
		int32 vid = MeshOut.AppendVertex(FVector3d(Position));
		ensure(vid == vi);

		FVector4f Normal = VertexBuffer.VertexTangentZ(vi);
		int32 nid = Normals->AppendElement(&Normal.X);
		ensure(nid == vi);

		FVector3f Tangent = VertexBuffer.VertexTangentX(vi);
		int32 tid = Tangents->AppendElement(&Tangent.X);
		ensure(tid == vi);

		FVector3f BiTangent = VertexBuffer.VertexTangentY(vi);
		int32 bid = BiTangents->AppendElement(&BiTangent.X);
		ensure(bid == vi);

		for (int32 ui = 0; ui < NumUVChannels; ++ui)
		{
			FVector2f UV = (bHasUVs) ? VertexBuffer.GetVertexUV(vi, ui) : FVector2f::ZeroVector;
			int32 uvid = UVs[ui]->AppendElement(UV);
			ensure(uvid == vi);
		}
	}


	int32 NumTriangles = IndexData.GetNumIndices() / 3;
	TArray<int> ToTIDMap; // maps original triangle id to dynamic mesh triangle id
	ToTIDMap.Init(FDynamicMesh3::InvalidID, NumTriangles);
	for (int32 ti = 0; ti < NumTriangles; ++ti)
	{
		int32 a = IndexData.GetIndex(3 * ti);
		int32 b = IndexData.GetIndex(3 * ti + 1);
		int32 c = IndexData.GetIndex(3 * ti + 2);
		int32 tid = MeshOut.AppendTriangle(a, b, c);
		
		//-- already seen this triangle for some reason.. or a degenerate tri
		if (tid == FDynamicMesh3::DuplicateTriangleID || tid == FDynamicMesh3::InvalidID)
		{
			continue;
		}

		// if append failed due to non-manifold, split verts
		if (tid == FDynamicMesh3::NonManifoldID)
		{
			int32 tri[3] = { a, b, c };
			int e0 = MeshOut.FindEdge(tri[0], tri[1]);
			int e1 = MeshOut.FindEdge(tri[1], tri[2]);
			int e2 = MeshOut.FindEdge(tri[2], tri[0]);
			// identify verts to split
			bool bToSplit[3] = { false, false, false };
			if (e0 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e0) == false)
			{
				bToSplit[0] = true;
				bToSplit[1] = true;
			}
			if (e1 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e1) == false)
			{
				bToSplit[1] = true;
				bToSplit[2] = true;
			}
			if (e2 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e2) == false)
			{
				bToSplit[2] = true;
				bToSplit[0] = true;
			}
			// split verts
			for (int32 i = 0; i < 3; ++i)
			{
				if (bToSplit[i]) // split verts
				{
					const int32 vi = tri[i];
					const FVector3f Position = PositionBuffer.VertexPosition(vi);
					const int32  newvi = MeshOut.AppendVertex((FVector3d)Position);
					const FVector4f Normal = VertexBuffer.VertexTangentZ(vi);
					const int32 nid = Normals->AppendElement(&Normal.X);
					ensure(nid == newvi);
					const FVector3f Tangent = VertexBuffer.VertexTangentX(vi);
					const int32 tnid = Tangents->AppendElement(&Tangent.X);
					ensure(tnid == newvi);
					const FVector3f BiTangent = VertexBuffer.VertexTangentY(vi);
					const int32 bid = BiTangents->AppendElement(&BiTangent.X);
					ensure(bid == newvi);
					for (int32 ui = 0; ui < NumUVChannels; ++ui)
					{
						const FVector2f UV = (bHasUVs) ? VertexBuffer.GetVertexUV(vi, ui) : FVector2f::ZeroVector;
						const int32 uvid = UVs[ui]->AppendElement(UV);
						ensure(uvid == newvi);
					}
					tri[i] = newvi;
				}
			}
		
			// try again, should always work
			tid = MeshOut.AppendTriangle(tri[0], tri[1], tri[2]);

			if (tid == FDynamicMesh3::NonManifoldID) // should never happen
			{
				ensure(0);
				continue;
			}
			a = tri[0];
			b = tri[1];
			c = tri[2];
		}
		// from source triangle id to dynamic mesh tid
		ToTIDMap[ti] = tid;

		Normals->SetTriangle(tid, FIndex3i(a, b, c));
		Tangents->SetTriangle(tid, FIndex3i(a, b, c));
		BiTangents->SetTriangle(tid, FIndex3i(a, b, c));
		for (int32 ui = 0; ui < NumUVChannels; ++ui)
		{
			UVs[ui]->SetTriangle(tid, FIndex3i(a, b, c));
		}
	}

	for (const FStaticMeshSection& Section : SectionData)
	{
		int32 MatIndex = Section.MaterialIndex;
		for (uint32 k = 0; k < Section.NumTriangles; ++k)
		{
			const uint32 TriIdx = (Section.FirstIndex / 3) + k;
			const int32 TID = ToTIDMap[static_cast<int32>(TriIdx)];
			if (TID != FDynamicMesh3::InvalidID)  // degenerate or duplicate triangles were not copied into MeshOut
			{
				MaterialID->SetValue(TID, MatIndex);
			}
		}
	}


	if (bAttemptToWeldSeams)
	{
		FMergeCoincidentMeshEdges Merge(&MeshOut);
		Merge.Apply();
	}

	return true;
}