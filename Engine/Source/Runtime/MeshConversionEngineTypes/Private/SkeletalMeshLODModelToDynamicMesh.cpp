// Copyright Epic Games, Inc. All Rights Reserved. 

#include "SkeletalMeshLODModelToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "ToDynamicMesh.h"

using namespace UE::Geometry;

#if WITH_EDITOR

class FSkeletalMeshLODModelWrapper
{
public:

	typedef int32 TriIDType;
	typedef int32 VertIDType;
	typedef int32 WedgeIDType;

	typedef int32 UVIDType;
	typedef int32 NormalIDType;
	typedef int32 ColorIDType;

	FSkeletalMeshLODModelWrapper(const FSkeletalMeshLODModel& MeshIn, bool bUseDisabledSections) :
		bIncludeDisabledSections(bUseDisabledSections),
		Mesh(&MeshIn)
	{
		auto SkipSection = [bUseDisabledSections](const FSkelMeshSection& Section)->bool
		{
			return (!bUseDisabledSections && Section.bDisabled);
		};
	
		// pre-count triangles and verts
		int32 TriCount = 0;
		int32 VertCount = 0;
		for (const FSkelMeshSection& Section : Mesh->Sections)
		{
			if (SkipSection(Section))
			{
				continue;
			}

			VertCount += (int32)Section.NumVertices;
			TriCount += (int32)Section.NumTriangles;
		}
		
		// construct a list of all valid VertIDs for this mesh.
		VertIDs.Reserve(VertCount);
		for (const FSkelMeshSection& Section : Mesh->Sections)
		{
			if (SkipSection(Section))
			{
				continue;
			}

			const int32 BaseVertexIndex = Section.BaseVertexIndex;
			const int32 NumSectionVtx = Section.SoftVertices.Num();
			for (int32 SectionVtxIndex = 0, VtxIndex = BaseVertexIndex; SectionVtxIndex < NumSectionVtx; ++SectionVtxIndex, ++VtxIndex)
			{
				VertIDs.Add(VtxIndex);
			}
		}

		// construct list of all valid TriIDs for this mesh.  
		TriIDs.Reserve(TriCount);
		for (int32 SectionIdx = 0; SectionIdx < Mesh->Sections.Num(); ++SectionIdx)
		{
			const FSkelMeshSection& Section = Mesh->Sections[SectionIdx];

			if (SkipSection(Section))
			{
				continue;
			}
			const int32 BaseIndex = Section.BaseIndex;
			const int32 BaseTriIdx = BaseIndex / 3;
			const int32 NumSectionTris = (int32)Section.NumTriangles;
			for (int32 i = 0, j = BaseTriIdx; i < NumSectionTris; ++i, ++j)
			{
				TriIDs.Add(j);
			}
		}


		const int32 MaxVertID = VertIDs.Last();
		VertIDToSectionID.Init(-1, MaxVertID + 1);// value only used for debugging 
		for (int32 SectionIdx = 0; SectionIdx < Mesh->Sections.Num(); ++SectionIdx)
		{
			const FSkelMeshSection& Section = Mesh->Sections[SectionIdx];
			if (SkipSection(Section))
			{
				continue;
			}

			const int32 BaseVertexIndex = (int32)Section.BaseVertexIndex;
			const int32 NumSectionVtx = Section.SoftVertices.Num();
			for (int32 SectionVtxIndex = 0, VtxIndex = BaseVertexIndex; SectionVtxIndex < NumSectionVtx; ++SectionVtxIndex, ++VtxIndex)
			{
				VertIDToSectionID[VtxIndex] = SectionIdx;
			}
		}

		const int32 MaxTriID = TriIDs.Last();
		TriIDToSectionID.Init(-1, MaxTriID + 1);
		for (int32 SectionIdx = 0; SectionIdx < Mesh->Sections.Num(); ++SectionIdx)
		{
			const FSkelMeshSection& Section = Mesh->Sections[SectionIdx];
			if (SkipSection(Section))
			{
				continue;
			}

			const int32 BaseIndex = (int32)Section.BaseIndex;
			const int32 BaseTriID = BaseIndex / 3;
			const int32 NumSectionTris = (int32)Section.NumTriangles;
			for (int32 i = 0; i < NumSectionTris; ++i)
			{
				TriIDToSectionID[i + BaseTriID] = SectionIdx;
			}
		}

	}

	
	int32 NumTris() const
	{
		return TriIDs.Num();
	}

	int32 NumVerts() const
	{
		return VertIDs.Num();
	}

	int32 NumUVLayers() const
	{
		return Mesh->NumTexCoords;
	}

	int32 NumWeightMapLayers() const
	{
		return 0;
	}

	float GetVertexWeight(int32, int32) const
	{
		ensure(false);
		return 0.0f;
	}

	FName GetWeightMapName(int32 LayerIndex) const
	{
		return FName();
	}


	// --"Vertex Buffer" info
	const TArray<VertIDType>& GetVertIDs() const
	{
		return VertIDs;
	}
	FVector3d GetPosition(VertIDType VtxID) const
	{
		int32 SectionID = VertIDToSectionID[VtxID];
		checkSlow(SectionID != -1);
		const FSkelMeshSection& Section = Mesh->Sections[SectionID];
		const int32 BaseVertexIndex = (int32)Section.BaseVertexIndex;

		return FVector3d(Section.SoftVertices[VtxID - BaseVertexIndex].Position);
	}

	// --"Index Buffer" info
	const TArray<TriIDType>& GetTriIDs() const
	{
		return TriIDs;
	}
	bool GetTri(TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
	{
		int32 Offset = TriID * 3;
		VID0 = Mesh->IndexBuffer[Offset];
		VID1 = Mesh->IndexBuffer[Offset + 1];
		VID2 = Mesh->IndexBuffer[Offset + 2];

		// maybe check IndexBuffer.Num() > Offset+2 ? 
		return true;
	}
	
	bool HasNormals() const
	{
		return true;
	}

	bool HasTangents() const
	{
		return true;
	}

	bool HasBiTangents() const
	{
		return true;
	}

	bool HasColors() const
	{
		return true;
	}

	//-- Access to per-wedge attributes --//
	void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const
	{
		int32 Offset = 3 * TriID;
		WID0 = Offset;
		WID1 = Offset + 1;
		WID2 = Offset + 2;
	}

	FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const
	{
		return (FVector2f)GetWedgeVertexInstance(WID).UVs[UVLayerIndex];	
	}

	FVector3f GetWedgeNormal(WedgeIDType WID) const
	{
		const FVector4f& TangentZ = GetWedgeVertexInstance(WID).TangentZ;
		return FVector3f(TangentZ.X, TangentZ.Y, TangentZ.Z);
	}

	FVector3f GetWedgeTangent(WedgeIDType WID) const
	{
		return FVector3f(GetWedgeVertexInstance(WID).TangentX);
	}

	FVector3f GetWedgeBiTangent(WedgeIDType WID) const
	{
		return FVector3f(GetWedgeVertexInstance(WID).TangentY);
	}
	
	FVector4f GetWedgeColor(WedgeIDType WID) const
	{
		FLinearColor LinearColor = GetWedgeVertexInstance(WID).Color.ReinterpretAsLinear();
		return FVector4f(LinearColor.R, LinearColor.G, LinearColor.B, LinearColor.A);
	}


	int32 GetMaterialIndex(TriIDType TriID) const
	{
		int32 SectionID = TriIDToSectionID[TriID];
		const FSkelMeshSection& Section = Mesh->Sections[SectionID];
		return Section.MaterialIndex;
	}

	
	
	//-- null implementation of shared attributes: Skeletal mesh model doesn't use these --//
	const TArray<int32>& GetUVIDs(int32 LayerID) const { return EmptyArray; }
	FVector2f GetUV(int32 LayerID, UVIDType UVID) const { check(0); return FVector2f(); }
	bool GetUVTri(int32 LayerID, const TriIDType&, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const { ID0 = ID1 = ID2 = UVIDType(-1); return false;}
	
	const TArray<int32>& GetNormalIDs() const { return EmptyArray; }
	FVector3f GetNormal(NormalIDType ID) const { check(0); return FVector3f(); }
	bool GetNormalTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }

	const TArray<int32>& GetTangentIDs() const { return EmptyArray; }
	FVector3f GetTangent(NormalIDType ID) const { check(0); return FVector3f(); }
	bool GetTangentTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }

	
	const TArray<int32>& GetBiTangentIDs() const { return EmptyArray; }
	FVector3f GetBiTangent(NormalIDType ID) const {check(0); return FVector3f(); }
	bool GetBiTangentTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }

	const TArray<int32>& GetColorIDs() const { return EmptyArray; }
	FVector4f GetColor(ColorIDType ID) const { check(0); return FVector4f(); }
	bool GetColorTri(const TriIDType&, ColorIDType& ID0, ColorIDType& ID1, ColorIDType& ID2) const { ID0 = ID1 = ID2 = ColorIDType(-1); return false; }

	// TODO: This file is only used in an outdated geometry blueprint library that is not being used. So adding an empty 
	// implementation for skin weights and bones for now. See SkinnedMeshToDynamicMesh.h/cpp for reference.
	int32 NumSkinWeightAttributes() const { return 0; }
	UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VertexID) const { return UE::AnimationCore::FBoneWeights(); }
	FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const { return NAME_None; }

	int32 GetNumBones() const { return 0; }
	FName GetBoneName(int32 BoneIdx) const { return NAME_None; }
	int32 GetBoneParentIndex(int32 BoneIdx) const { return INDEX_NONE; }
	FTransform GetBonePose(int32 BoneIdx) const { return FTransform::Identity; }
	FVector4f GetBoneColor(int32 BoneIdx) const { return FVector4f::One(); }
	
	// -- additional methods, not required by the conversion interface
	const  FSkeletalMeshLODModel& GetSrcMesh() const
	{
		return *Mesh;
	}


private:

	inline const FSoftSkinVertex& GetWedgeVertexInstance(WedgeIDType WID) const
	{
		int32 VertID = GetVertID(WID);
		int32 SectionID = VertIDToSectionID[VertID];
		checkSlow(SectionID != -1);
		const FSkelMeshSection& Section = Mesh->Sections[SectionID];
		const int32 BaseVertexIndex = (int32)Section.BaseVertexIndex;
		return Section.SoftVertices[VertID - BaseVertexIndex];
	}

	FSkeletalMeshLODModelWrapper();
	FSkeletalMeshLODModelWrapper(const FSkeletalMeshLODModelWrapper&);

	TArray<int32> EmptyArray;
	
	// Convert the WedgeID the ID of the corresponding position vertex.
	inline const VertIDType GetVertID(int32 WedgeID) const
	{
		return Mesh->IndexBuffer[WedgeID];
	}

private:

	bool bIncludeDisabledSections;
	TArray<VertIDType> VertIDToSectionID;
	TArray<TriIDType> TriIDToSectionID;
	TArray<TriIDType> TriIDs;
	TArray<VertIDType> VertIDs;
	
	const FSkeletalMeshLODModel* Mesh;
};





void FSkeletalMeshLODModelToDynamicMesh::Convert(const FSkeletalMeshLODModel* MeshIn, FDynamicMesh3& MeshOut, bool bCopyTangents)
{

	

	const bool bIncludeDisabledSections = true;
	FSkeletalMeshLODModelWrapper ModelWrapper(*MeshIn, bIncludeDisabledSections);

	if (bPrintDebugMessages)
	{
		UE_LOGF(LogTemp, Warning, "FSkeletalMeshLODModelToDynamicMesh:FSkeletalMeshLODModel verts %d  instances %d", ModelWrapper.NumVerts(), 3*ModelWrapper.NumTris());
	}

	// Making default GroupIDs for the mesh
	auto TriToGroupID = [&ModelWrapper](const int32& SrcTriID)->int32 {return (ModelWrapper.GetMaterialIndex(SrcTriID) + 1); };
	
	// Actual conversion.
	TToDynamicMesh<FSkeletalMeshLODModelWrapper> SkeletalToDynamicMesh;
	if (bDisableAttributes)
	{ 
		SkeletalToDynamicMesh.ConvertWOAttributes(MeshOut, ModelWrapper, TriToGroupID );
	}
	else
	{ 
		auto TriToMaterialID = [&ModelWrapper](const int32& SrcTriID)->int32 { return ModelWrapper.GetMaterialIndex(SrcTriID); };
		SkeletalToDynamicMesh.Convert(MeshOut, ModelWrapper, TriToGroupID, TriToMaterialID, bCopyTangents);						
	}
	

	if (!bEnableOutputGroups)
	{
		MeshOut.DiscardTriangleGroups();
	}
	FDateTime Time_AfterAttribs = FDateTime::Now();

	// move maps to the calling class or let them get destroyed when SkeletalToDynamicMesh goes out of scope
	if (bCalculateMaps)
	{
		Swap(TriIDMap, SkeletalToDynamicMesh.ToSrcTriIDMap);
		Swap(VertIDMap, SkeletalToDynamicMesh.ToSrcVertIDMap);
	}

	
	if (bPrintDebugMessages)
	{
		int NumUVLayers = ModelWrapper.NumUVLayers();
		UE_LOGF(LogTemp, Warning, "FSkeletalMeshLODModelToDynamicMesh:  Conversion Timing: Triangles %fs   Attributes %fs",
			(SkeletalToDynamicMesh.Time_AfterTriangles - SkeletalToDynamicMesh.Time_AfterVertices).GetTotalSeconds(), 
			(Time_AfterAttribs   - SkeletalToDynamicMesh.Time_AfterTriangles).GetTotalSeconds());

		
		int NumUVs = (MeshOut.HasAttributes() && NumUVLayers > 0) ? MeshOut.Attributes()->PrimaryUV()->MaxElementID() : 0;
		int NumNormals = 0; 
		if (MeshOut.HasAttributes())
		{
			(MeshOut.Attributes()->PrimaryNormals() != nullptr) ? MeshOut.Attributes()->PrimaryNormals()->MaxElementID() : 0;
		}

		UE_LOGF(LogTemp, Warning, "FSkeletalMeshLODModelToDynamicMesh:  FDynamicMesh verts %d triangles %d (primary) uvs %d normals %d", MeshOut.MaxVertexID(), MeshOut.MaxTriangleID(), NumUVs, NumNormals);
	}
			
}

#else

void FSkeletalMeshLODModelToDynamicMesh::Convert(const FSkeletalMeshLODModel* MeshIn, FDynamicMesh3& MeshOut, bool bCopyTangents)
{
	// Conversion only supported with editor.
	check(0);
}

#endif  // end with editor
