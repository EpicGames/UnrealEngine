// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshLODRenderDataToDynamicMesh.h"
#include "SkeletalMeshLODRenderDataMeshAdapter.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "SkeletalMeshAttributes.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "ReferenceSkeleton.h"

using namespace UE::Geometry;

bool FSkeletalMeshLODRenderDataToDynamicMesh::Convert(
	const FSkeletalMeshLODRenderData* SkeletalMeshResources,
	const FReferenceSkeleton& RefSkeleton,
	const ConversionOptions& Options,
	FDynamicMesh3& OutputMesh,
	bool bHasVertexColors,
	TFunctionRef<FColor(int32)> GetVertexColorFromLODVertexIndex)
{
	if (!ensureMsgf(SkeletalMeshResources && SkeletalMeshResources->StaticVertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess(), TEXT("bAllowCPUAccess must be set to true for SkeletalMesh before calling FSkeletalMeshLODRenderDataToDynamicMesh::Convert(), otherwise the mesh geometry data isn't accessible!")))
	{
		return false;
	}

	FSkeletalMeshLODRenderDataMeshAdapter Adapter(SkeletalMeshResources);

	Adapter.SetBuildScale(Options.BuildScale, false);

	OutputMesh = FDynamicMesh3();
	OutputMesh.EnableTriangleGroups();
	if (Options.bWantNormals || Options.bWantTangents || Options.bWantUVs || Options.bWantVertexColors || Options.bWantMaterialIDs)
	{
		OutputMesh.EnableAttributes();
	}

	// map from FSkeletalMeshLODRenderData triangle ID to DynamicMesh TID
	TArray<int32> ToDstTriID;

	/**
	* map from DynamicMesh vertex Id to FSkeletalMeshLODRenderData vertex id.
	* NB: due to vertex splitting, multiple DynamicMesh vertex ids
	* may map to the same FSkeletalMeshLODRenderData vertex id.
	*  ( a vertex split is a result of reconciling non-manifold MeshDescription vertex )
	*/
	TArray<int32> ToSrcVID;

	// Copy vertices. LODMesh is dense so this should be 1-1
	int32 SrcVertexCount = Adapter.VertexCount();
	ToSrcVID.SetNumUninitialized(SrcVertexCount);
	for (int32 SrcVertID = 0; SrcVertID < SrcVertexCount; ++SrcVertID)
	{
		FVector3d Position = Adapter.GetVertex(SrcVertID);
		int32 DstVertID = OutputMesh.AppendVertex(Position);
		ToSrcVID[DstVertID] = SrcVertID;

		if (DstVertID != SrcVertID)
		{
			// should only happen in the source mesh is missing vertices. 
			OutputMesh.Clear();
			ensure(false);
			return false;
		}
	}

	// Copy triangles. LODMesh is dense so this should be 1-1 unless there is a duplicate tri or non-manifold edge 
	int32 SrcTriangleCount = Adapter.TriangleCount();
	ToDstTriID.Init(FDynamicMesh3::InvalidID, SrcTriangleCount);
	for (int32 SrcTriID = 0; SrcTriID < SrcTriangleCount; ++SrcTriID)
	{
		FIndex3i Tri = Adapter.GetTriangle(SrcTriID);
		int32 DstTriID = OutputMesh.AppendTriangle(Tri.A, Tri.B, Tri.C);

		if (DstTriID == FDynamicMesh3::DuplicateTriangleID || DstTriID == FDynamicMesh3::InvalidID)
		{
			continue;
		}

		// split verts on the non-manifold edge(s)
		if (DstTriID == FDynamicMesh3::NonManifoldID)
		{
			int e01 = OutputMesh.FindEdge(Tri[0], Tri[1]);
			int e12 = OutputMesh.FindEdge(Tri[1], Tri[2]);
			int e20 = OutputMesh.FindEdge(Tri[2], Tri[0]);

			// determine which verts need to be duplicated
			bool bToSplit[3] = { false, false, false };
			if (e01 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e01) == false)
			{
				bToSplit[0] = true;
				bToSplit[1] = true;
			}
			if (e12 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e12) == false)
			{
				bToSplit[1] = true;
				bToSplit[2] = true;
			}
			if (e20 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e20) == false)
			{
				bToSplit[2] = true;
				bToSplit[0] = true;
			}
			for (int32 i = 0; i < 3; ++i)
			{
				if (bToSplit[i])
				{
					const int32 SrcVID = Tri[i];
					const FVector3d Position = Adapter.GetVertex(SrcVID);
					const int32 NewDstVertIdx = OutputMesh.AppendVertex(Position);
					Tri[i] = NewDstVertIdx;
					ToSrcVID.SetNumUninitialized(NewDstVertIdx + 1);
					ToSrcVID[NewDstVertIdx] = SrcVID;
				}
			}

			DstTriID = OutputMesh.AppendTriangle(Tri[0], Tri[1], Tri[2]);
		}

		ToDstTriID[SrcTriID] = DstTriID;

	}

	// transfer sections to PolyGroups and MaterialIDs
	FDynamicMeshMaterialAttribute* MaterialIDs = nullptr;
	if (Options.bWantMaterialIDs)
	{
		OutputMesh.Attributes()->EnableMaterialID();
		MaterialIDs = OutputMesh.Attributes()->GetMaterialID();
	}
	for (int32 SectionIdx = 0; SectionIdx < SkeletalMeshResources->RenderSections.Num(); ++SectionIdx)
	{
		const FSkelMeshRenderSection& Section = SkeletalMeshResources->RenderSections[SectionIdx];
		for (uint32 TriIdx = 0; TriIdx < Section.NumTriangles; ++TriIdx)
		{
			int32 SrcTriangleID = (int32)(Section.BaseIndex / 3 + TriIdx);
			int32 DstTriID = ToDstTriID[SrcTriangleID];
			if (DstTriID != FDynamicMesh3::InvalidID)
			{
				OutputMesh.SetTriangleGroup(DstTriID, SectionIdx);
				if (MaterialIDs != nullptr)
				{
					int32 MaterialIndex = (int32)Section.MaterialIndex;
					MaterialIDs->SetValue(DstTriID, MaterialIndex);
				}
			}
		}
	}

	// Helper to copy a per-vertex attribute to an overlay
	auto CopyPerVertexOverlay = [SrcTriangleCount, &ToSrcVID, &ToDstTriID, &OutputMesh](auto* Overlay, auto GetElement)
		{
			const int32 DstVertexCount = ToSrcVID.Num();
			for (int32 DstVertID = 0; DstVertID < DstVertexCount; ++DstVertID)
			{
				const int32 SrcVID = ToSrcVID[DstVertID];
				auto Element = GetElement(SrcVID);
				int32 ElemID = Overlay->AppendElement(Element);
				check(ElemID == DstVertID);
			}

			for (int32 SrcTriID = 0; SrcTriID < SrcTriangleCount; ++SrcTriID)
			{
				const int32 DstTriID = ToDstTriID[SrcTriID];
				if (DstTriID != FDynamicMesh3::InvalidID)
				{
					FIndex3i Tri = OutputMesh.GetTriangle(DstTriID);
					Overlay->SetTriangle(DstTriID, FIndex3i(Tri.A, Tri.B, Tri.C));
				}
			}
		};

	// copy overlay normals
	if (Adapter.HasNormals() && Options.bWantNormals)
	{
		FDynamicMeshNormalOverlay* Normals = OutputMesh.Attributes()->PrimaryNormals();
		if (Normals != nullptr)
		{
			CopyPerVertexOverlay(Normals, [&](int32 SrcVID)->FVector3f { return Adapter.GetNormal(SrcVID); });
		}
	}

	// copy overlay tangents
	if (Adapter.HasNormals() && Options.bWantTangents)
	{
		OutputMesh.Attributes()->EnableTangents();
		FDynamicMeshNormalOverlay* TangentsX = OutputMesh.Attributes()->PrimaryTangents();
		if (TangentsX != nullptr)
		{
			CopyPerVertexOverlay(TangentsX, [&](int32 SrcVID)->FVector3f { return Adapter.GetTangentX(SrcVID); });
		}

		FDynamicMeshNormalOverlay* TangentsY = OutputMesh.Attributes()->PrimaryBiTangents();
		if (TangentsY != nullptr)
		{
			CopyPerVertexOverlay(TangentsY, [&](int32 SrcVID)->FVector3f { return Adapter.GetTangentY(SrcVID); });
		}
	}

	// copy UV layers
	if (Adapter.HasUVs() && Options.bWantUVs)
	{
		int32 NumUVLayers = Adapter.NumUVLayers();
		if (NumUVLayers > 0)
		{
			OutputMesh.Attributes()->SetNumUVLayers(NumUVLayers);
			for (int32 UVLayerIndex = 0; UVLayerIndex < NumUVLayers; ++UVLayerIndex)
			{
				FDynamicMeshUVOverlay* UVOverlay = OutputMesh.Attributes()->GetUVLayer(UVLayerIndex);
				CopyPerVertexOverlay(UVOverlay, [&](int32 SrcVID)->FVector2f { return Adapter.GetUV(SrcVID, UVLayerIndex); });
			}
		}
	}

	// copy overlay colors
	if (bHasVertexColors && Options.bWantVertexColors)
	{
		OutputMesh.Attributes()->EnablePrimaryColors();
		FDynamicMeshColorOverlay* Colors = OutputMesh.Attributes()->PrimaryColors();

		CopyPerVertexOverlay(Colors, [&](int32 SrcVID)->FVector4f { return GetVertexColorFromLODVertexIndex(SrcVID).ReinterpretAsLinear(); });
	}

	// copy skin weights
	if (Adapter.HasSkinWeights() && Options.bWantSkinWeights)
	{
		auto CopyPerVertexSkinWeights = [&ToSrcVID, &OutputMesh, &Adapter](FDynamicMeshVertexSkinWeightsAttribute* Overlay)
		{
			const int32 DstVertexCount = ToSrcVID.Num();
			for (int32 DstVertID = 0; DstVertID < DstVertexCount; ++DstVertID)
			{
				const int32 SrcVID = ToSrcVID[DstVertID];
				const FSkinWeightInfo WeightInfo = Adapter.GetSkinWeightInfo(SrcVID);;
				const UE::AnimationCore::FBoneWeights Weights = UE::AnimationCore::FBoneWeights::Create(WeightInfo.InfluenceBones, WeightInfo.InfluenceWeights);
				Overlay->SetValue(DstVertID, Weights);
			}
		};

		
		FDynamicMeshVertexSkinWeightsAttribute* SkinAttribute = new FDynamicMeshVertexSkinWeightsAttribute(&OutputMesh);

		CopyPerVertexSkinWeights(SkinAttribute);
		
		SkinAttribute->SetName(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);

		OutputMesh.Attributes()->AttachSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName, SkinAttribute);

		// populate bones attribute
		const int32 NumBones = RefSkeleton.GetRawBoneNum();
		if (NumBones)
		{
			OutputMesh.Attributes()->EnableBones(NumBones);
			FDynamicMeshBoneNameAttribute* BoneNameAttrib = OutputMesh.Attributes()->GetBoneNames();
			FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = OutputMesh.Attributes()->GetBoneParentIndices();
			FDynamicMeshBonePoseAttribute* BonePoses = OutputMesh.Attributes()->GetBonePoses();
			FDynamicMeshBoneColorAttribute* BoneColors = OutputMesh.Attributes()->GetBoneColors();

			for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
			{
				BoneNameAttrib->SetValue(BoneIdx, RefSkeleton.GetRawRefBoneInfo()[BoneIdx].Name);
				BoneParentIndices->SetValue(BoneIdx, RefSkeleton.GetRawRefBoneInfo()[BoneIdx].ParentIndex);
				BonePoses->SetValue(BoneIdx, RefSkeleton.GetRawRefBonePose()[BoneIdx]);
				BoneColors->SetValue(BoneIdx, FVector4f::One());
			}
		}
	}


	return true;
}


bool FSkeletalMeshLODRenderDataToDynamicMesh::Convert(
	const FSkeletalMeshLODRenderData* SkeletalMeshResources,
	const FReferenceSkeleton& RefSkeleton,
	const ConversionOptions& Options,
	FDynamicMesh3& OutputMesh)
{
	bool bHasVertexColors = SkeletalMeshResources && SkeletalMeshResources->StaticVertexBuffers.ColorVertexBuffer.GetAllowCPUAccess();
	return Convert(SkeletalMeshResources, RefSkeleton, Options, OutputMesh, bHasVertexColors,
		[SkeletalMeshResources](int32 VID)
		{
			return SkeletalMeshResources->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VID);
		});
}
