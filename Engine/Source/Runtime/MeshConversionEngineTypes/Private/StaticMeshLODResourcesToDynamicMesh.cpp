// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshLODResourcesToDynamicMesh.h"
#include "StaticMeshLODResourcesAdapter.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"


using namespace UE::Geometry;

bool FStaticMeshLODResourcesToDynamicMesh::Convert(
	const FStaticMeshLODResources* StaticMeshResources,
	const ConversionOptions& Options,
	FDynamicMesh3& OutputMesh,
	bool bHasVertexColors,
	TFunctionRef<FColor(int32)> GetVertexColorFromLODVertexIndex)
{
	if (!ensureMsgf(StaticMeshResources && StaticMeshResources->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess(), TEXT("bAllowCPUAccess must be set to true for StaticMeshes before calling FStaticMeshLODResourcesToDynamicMesh::Convert(), otherwise the mesh geometry data isn't accessible!")))
	{
		return false;
	}

	FStaticMeshLODResourcesMeshAdapter Adapter(StaticMeshResources);

	Adapter.SetBuildScale(Options.BuildScale, false);

	OutputMesh = FDynamicMesh3();
	OutputMesh.EnableTriangleGroups();
	if (Options.bWantNormals || Options.bWantTangents || Options.bWantUVs || Options.bWantVertexColors || Options.bWantMaterialIDs)
	{
		OutputMesh.EnableAttributes();
	}

	// map from StaticMeshResouce triangle ID to DynamicMesh TID
	TArray<int32> ToDstTriID;

	/**
	* map from DynamicMesh vertex Id to StaticMeshResouce vertex id.
	* NB: due to vertex splitting, multiple DynamicMesh vertex ids
	* may map to the same StaticMeshResource vertex id.
	*  ( a vertex split is a result of reconciling non-manifold MeshDescription vertex )
	*/
	TArray<int32> ToSrcVID;

	// Copy vertices. LODMesh is dense so this should be 1-1
	int32 SrcVertexCount = Adapter.VertexCount();
	ToSrcVID.SetNumUninitialized(SrcVertexCount);
	for ( int32 SrcVertID = 0; SrcVertID < SrcVertexCount; ++SrcVertID )
	{
		FVector3d Position = Adapter.GetVertex(SrcVertID);
		int32 DstVertID = OutputMesh.AppendVertex(Position);
		ToSrcVID[DstVertID] = SrcVertID;
		
		if (DstVertID != SrcVertID)
		{
			// should only happen in the source mesh is missing vertices. 
			OutputMesh.Clear();
			ensure(false);
			return false;
		}
	}

	// Copy triangles. LODMesh is dense so this should be 1-1 unless there is a duplicate tri or non-manifold edge 
	int32 SrcTriangleCount = Adapter.TriangleCount();
	ToDstTriID.Init(FDynamicMesh3::InvalidID, SrcTriangleCount);
	for (int32 SrcTriID = 0; SrcTriID < SrcTriangleCount; ++SrcTriID)
	{
		FIndex3i Tri = Adapter.GetTriangle(SrcTriID);
		int32 DstTriID = OutputMesh.AppendTriangle(Tri.A, Tri.B, Tri.C);

		if (DstTriID == FDynamicMesh3::DuplicateTriangleID || DstTriID == FDynamicMesh3::InvalidID)
		{
			continue;
		}

		// split verts on the non-manifold edge(s)
		if (DstTriID == FDynamicMesh3::NonManifoldID)
		{
			int e01 = OutputMesh.FindEdge(Tri[0], Tri[1]);
			int e12 = OutputMesh.FindEdge(Tri[1], Tri[2]);
			int e20 = OutputMesh.FindEdge(Tri[2], Tri[0]);

			// determine which verts need to be duplicated
			bool bToSplit[3] = { false, false, false };
			if (e01 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e01) == false)
			{
				bToSplit[0] = true;
				bToSplit[1] = true;
			}
			if (e12 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e12) == false)
			{
				bToSplit[1] = true;
				bToSplit[2] = true;
			}
			if (e20 != FDynamicMesh3::InvalidID && OutputMesh.IsBoundaryEdge(e20) == false)
			{
				bToSplit[2] = true;
				bToSplit[0] = true;
			}
			for (int32 i = 0; i < 3; ++i)
			{
				if (bToSplit[i])
				{
					const int32 SrcVID = Tri[i];
					const FVector3d Position = Adapter.GetVertex(SrcVID);
					const int32 NewDstVertIdx = OutputMesh.AppendVertex(Position);
					Tri[i] = NewDstVertIdx;
					ToSrcVID.SetNumUninitialized(NewDstVertIdx + 1);
					ToSrcVID[NewDstVertIdx] = SrcVID;
				}
			}

			DstTriID = OutputMesh.AppendTriangle(Tri[0], Tri[1], Tri[2]);
		}

		ToDstTriID[SrcTriID] = DstTriID;

	}

	// transfer sections to PolyGroups and MaterialIDs
	FDynamicMeshMaterialAttribute* MaterialIDs = nullptr;
	if (Options.bWantMaterialIDs)
	{
		OutputMesh.Attributes()->EnableMaterialID();
		MaterialIDs = OutputMesh.Attributes()->GetMaterialID();
	}
	for ( int32 SectionIdx = 0; SectionIdx < StaticMeshResources->Sections.Num(); ++SectionIdx)
	{
		const FStaticMeshSection& Section = StaticMeshResources->Sections[SectionIdx];
		for (uint32 TriIdx = 0; TriIdx < Section.NumTriangles; ++TriIdx)
		{
			int32 SrcTriangleID = (int32)(Section.FirstIndex/3 + TriIdx);
			int32 DstTriID = ToDstTriID[SrcTriangleID];
			if (DstTriID != FDynamicMesh3::InvalidID)
			{ 
				OutputMesh.SetTriangleGroup(DstTriID, SectionIdx);
				if (MaterialIDs != nullptr)
				{
					MaterialIDs->SetValue(DstTriID, Section.MaterialIndex);
				}
			}
		}
	}

	// Helper to copy a per-vertex attribute to an overlay
	auto CopyPerVertexOverlay = [SrcTriangleCount, &ToSrcVID, &ToDstTriID, &OutputMesh](auto* Overlay, auto GetElement)
	{
		const int32 DstVertexCount = ToSrcVID.Num();
		for (int32 DstVertID = 0; DstVertID < DstVertexCount; ++DstVertID)
		{
			const int32 SrcVID = ToSrcVID[DstVertID];
			auto Element = GetElement(SrcVID);
			int32 ElemID = Overlay->AppendElement(Element);
			check(ElemID == DstVertID);
		}

		for (int32 SrcTriID = 0; SrcTriID < SrcTriangleCount; ++SrcTriID)
		{
			const int32 DstTriID = ToDstTriID[SrcTriID];
			if (DstTriID != FDynamicMesh3::InvalidID)
			{
				FIndex3i Tri = OutputMesh.GetTriangle(DstTriID);
				Overlay->SetTriangle(DstTriID, FIndex3i(Tri.A, Tri.B, Tri.C));
			}
		}
	};

	// copy overlay normals
	if (Adapter.HasNormals() && Options.bWantNormals)
	{
		FDynamicMeshNormalOverlay* Normals = OutputMesh.Attributes()->PrimaryNormals();
		if (Normals != nullptr)
		{
			CopyPerVertexOverlay(Normals, [&](int32 SrcVID)->FVector3f { return Adapter.GetNormal(SrcVID); });
		}
	}

	// copy overlay tangents
	if ( Adapter.HasNormals() && Options.bWantTangents )
	{
		OutputMesh.Attributes()->EnableTangents();
		FDynamicMeshNormalOverlay* TangentsX = OutputMesh.Attributes()->PrimaryTangents();
		if (TangentsX != nullptr)
		{
			CopyPerVertexOverlay(TangentsX, [&](int32 SrcVID)->FVector3f { return Adapter.GetTangentX(SrcVID); });
		}

		FDynamicMeshNormalOverlay* TangentsY = OutputMesh.Attributes()->PrimaryBiTangents();
		if (TangentsY != nullptr)
		{
			CopyPerVertexOverlay(TangentsY, [&](int32 SrcVID)->FVector3f { return Adapter.GetTangentY(SrcVID); });
		}
	}

	// copy UV layers
	if (Adapter.HasUVs() && Options.bWantUVs)
	{
		int32 NumUVLayers = Adapter.NumUVLayers();
		if (NumUVLayers > 0)
		{
			OutputMesh.Attributes()->SetNumUVLayers(NumUVLayers);
			for (int32 UVLayerIndex = 0; UVLayerIndex < NumUVLayers; ++UVLayerIndex)
			{
				FDynamicMeshUVOverlay* UVOverlay = OutputMesh.Attributes()->GetUVLayer(UVLayerIndex);
				CopyPerVertexOverlay(UVOverlay, [&](int32 SrcVID)->FVector2f { return Adapter.GetUV(SrcVID, UVLayerIndex); });
			}
		}
	}

	// copy overlay colors
	if (bHasVertexColors && Options.bWantVertexColors)
	{
		OutputMesh.Attributes()->EnablePrimaryColors();
		FDynamicMeshColorOverlay* Colors = OutputMesh.Attributes()->PrimaryColors();

		CopyPerVertexOverlay(Colors, [&](int32 SrcVID)->FVector4f { return GetVertexColorFromLODVertexIndex(SrcVID).ReinterpretAsLinear(); });
	}

	return true;
}


bool FStaticMeshLODResourcesToDynamicMesh::Convert(
	const FStaticMeshLODResources* StaticMeshResources,
	const ConversionOptions& Options,
	FDynamicMesh3& OutputMesh)
{
	bool bHasVertexColors = StaticMeshResources && StaticMeshResources->VertexBuffers.ColorVertexBuffer.GetAllowCPUAccess();
	return Convert(StaticMeshResources, Options, OutputMesh, bHasVertexColors, 
		[StaticMeshResources](int32 VID)
		{
			return StaticMeshResources->VertexBuffers.ColorVertexBuffer.VertexColor(VID);
		});
}
