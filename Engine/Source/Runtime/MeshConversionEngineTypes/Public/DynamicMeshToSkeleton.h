// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USkeleton;
namespace UE::Geometry { class FDynamicMesh3; }

namespace UE::Conversion 
{
	using namespace UE::Geometry;

	/**
		* Verify if a dynamic mesh is compatible with a USkeleton
		*
		* @param InDynamicMesh  Dynamic mesh to compare the Skeleton against. Must have bone attributes enabled.
		* @param InSkeleton Skeleton asset to compare against the dynamic mesh skeletal attributes
		* @param bDoParentChainCheck  When true (the default) this method also compares if chains match with the parent.
		*
		* @return true if the skeleton is compatible with the supplied dynamic mesh, false if not.
		*/
	bool MESHCONVERSIONENGINETYPES_API IsDynamicMeshCompatibleWithSkeleton(const FDynamicMesh3& InDynamicMesh, const USkeleton& InSkeleton, bool bDoParentChainCheck = true);

}  // end namespace UE::Conversion

