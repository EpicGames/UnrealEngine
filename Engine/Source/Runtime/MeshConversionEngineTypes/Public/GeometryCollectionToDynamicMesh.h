// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryBase.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);
struct FManagedArrayCollection;
class FGeometryCollection;

namespace UE::Geometry
{

struct FGeometryCollectionToDynamicMeshes
{
	struct FMeshInfo
	{
		// Transform Index of the bone represented
		int32 TransformIndex = -1;
		// Mesh holding the geometry at (or under) a given bone, in a common coordinate space
		TUniquePtr<FDynamicMesh3> Mesh;
		// Stores the Transform that has *already* been applied to the Mesh vertices, which took it from local bone space to a common space
		FTransform Transform = FTransform::Identity;
	};

	TArray<FMeshInfo> Meshes;


	
	enum class EInvisibleFaceConversion : uint8
	{
		Skip,
		TagWithPolygroup
	};

	struct FToMeshOptions
	{
		FTransform Transform = FTransform::Identity;

		bool bWeldVertices = false;
		bool bSaveIsolatedVertices = false;

		bool bInternalFaceTagsAsPolygroups = true;
		EInvisibleFaceConversion InvisibleFaces = EInvisibleFaceConversion::Skip;
	};

	struct FToCollectionOptions
	{
		// Whether to set faces with no 'visible' tag as visible (if true) or invisible (if false)
		bool bDefaultFaceVisible = true;
		// Whether to set faces with no 'internal' tag as internal (if true) or external (if false)
		bool bDefaultFaceInternal = false;
		// Whether the appended geometry is allowed to be added as a root transform (if NewMeshParentIndex == -1)
		bool bAllowAppendAsRoot = false;

		// Parent index to use if adding a new mesh (with no existing transform) -- if invalid, will add to root
		int32 NewMeshParentIndex = -1;
	};

	// Convert all meshes in the given collection to dynamic meshes
	MESHCONVERSIONENGINETYPES_API bool Init(const FManagedArrayCollection& Collection, const FToMeshOptions& Options);

	// Convert the meshes in the collection at the given transforms to dynamic meshes
	MESHCONVERSIONENGINETYPES_API bool InitFromTransformSelection(const FManagedArrayCollection& Collection, TConstArrayView<int32> TransformIndices, const FToMeshOptions& Options);

	// Update a geometry collection with the current meshes array
	// Note this updates a Geometry Collection rather than the more general FManagedArrayCollection because some of the updating code is specific to geometry collection
	MESHCONVERSIONENGINETYPES_API bool UpdateGeometryCollection(FGeometryCollection& Collection, const FToCollectionOptions& Options) const;

	// Add a new mesh to the geometry collection
	// Note this updates a Geometry Collection rather than the more general FManagedArrayCollection because some of the updating code is specific to geometry collection
	// @return Index of added transform
	MESHCONVERSIONENGINETYPES_API static int32 AppendMeshToCollection(FGeometryCollection& Collection, const FDynamicMesh3& Mesh, const FTransform& MeshTransform, const FToCollectionOptions& Options);



	// Get the name used for the polygroup that we optionally set on the dynamic mesh, corresponding to the internal face tags of the geometry collection triangles
	static MESHCONVERSIONENGINETYPES_API FName InternalFacePolyGroupName();
	// Get the name used for the polygroup that we optionally set on the dynamic mesh, corresponding to the invisible face tags of the geometry collection triangles
	static MESHCONVERSIONENGINETYPES_API FName VisibleFacePolyGroupName();

private:
	
	// private helper method with shared implementation for all Init* methods
	// @param bTransformInComponentSpace If false, the Transforms array holds transforms that are relative to the corresponding bone parent. If true, the Transforms are all relative to a common coordinate space.
	bool InitHelper(const FManagedArrayCollection& Collection, bool bTransformInComponentSpace, TConstArrayView<FTransform3f> Transforms, bool bAllTransforms, TConstArrayView<int32> TransformIndices, const FToMeshOptions& Options);
	static TConstArrayView<FTransform3f> GetCollectionTransforms(const FManagedArrayCollection& Collection);

	// Update an existing geometry in a collection w/ a new mesh (w/ the same number of faces and vertices!)
	static bool UpdateCollection(const FTransform& FromCollection, const FDynamicMesh3& Mesh, int32 GeometryIdx, FGeometryCollection& Output, const FToCollectionOptions& Options);
};

} // namespace UE::Geometry::GeometryCollectionDynamicMeshConversion