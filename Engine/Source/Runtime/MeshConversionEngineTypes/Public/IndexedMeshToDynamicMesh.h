// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StaticMeshResources.h"
#include "DynamicMesh/DynamicMesh3.h"


namespace UE {
namespace Conversion {

using namespace UE::Geometry;

/**
 * Convert rendering buffer data (from FStaticMeshLODResources) to a DynamicMesh3.
 * The output FDynamicMesh3 attribute set will have as many UV channels as found in VertexData,
 * as well as overlay Tangents enabled/copied, and MaterialID set.
 * @param bAttemptToWeldSeams if true, try to weld seam edges together to create a more closed mesh using FMergeCoincidentMeshEdges
 */
bool MESHCONVERSIONENGINETYPES_API RenderBuffersToDynamicMesh(
	const FStaticMeshVertexBuffers& VertexData,
	const FRawStaticIndexBuffer& IndexData,
	const FStaticMeshSectionArray& SectionData,
	FDynamicMesh3& MeshOut,
	bool bAttemptToWeldSeams = false );




}  // end namespace Conversion
}  // end namespace UE


