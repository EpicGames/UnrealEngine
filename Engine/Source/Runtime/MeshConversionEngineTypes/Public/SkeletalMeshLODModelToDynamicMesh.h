// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

// forward 
class FSkeletalMeshLODModel;

using UE::Geometry::FDynamicMesh3;

/**
 * Convert FSkeletalMeshLODModel to FDynamicMesh3
 *
 */
class FSkeletalMeshLODModelToDynamicMesh
{
public:
	/** If true, will print some possibly-helpful debugging spew to output log */
	bool bPrintDebugMessages = false;

	/** Should we initialize triangle groups on output mesh */
	bool bEnableOutputGroups = true;

	/** Should we calculate conversion index maps */
	bool bCalculateMaps = true;

	/** Ignore all mesh attributes (e.g. UV/Normal layers, material groups) */
	bool bDisableAttributes = false;


	/** map from DynamicMesh triangle ID to FSkeletalMeshLODModel TriIdx*/
	TArray<int32> TriIDMap;

	/**
	* map from DynamicMesh vertex Id to  FSkeletalMeshLODModel FVertexID.
	* NB: due to vertex splitting, multiple DynamicMesh vertex ids
	* may map to the same  FSkeletalMeshLODModel FVertexID.
	*  ( a vertex split is a result of reconciling non-manifold  FSkeletalMeshLODModel vertex )
	*/
	TArray<int32> VertIDMap;


	/**
	 * Default conversion of MeshDescription to DynamicMesh
	 * @param bCopyTangents  - if bDisableAttributes is false, this requests the tangent plane vectors (tangent and bitangent)
	 *                          be stored as overlays in the MeshOut DynamicAttributeSet, provided they exist on the MeshIn
	 */
	MESHCONVERSIONENGINETYPES_API void Convert(const  FSkeletalMeshLODModel* MeshIn, FDynamicMesh3& MeshOut, bool bCopyTangents = false);

};
