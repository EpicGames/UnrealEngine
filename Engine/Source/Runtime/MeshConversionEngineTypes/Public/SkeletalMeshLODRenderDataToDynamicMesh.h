// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

class FSkeletalMeshLODRenderData;
struct FReferenceSkeleton;

namespace UE
{
namespace Geometry
{
/**
 * FSkeletalMeshLODRenderDataToDynamicMesh can be used to create a FDynamicMesh3 from
 * a FSkeletalMeshLODRenderData, which is the variant of the mesh in a SkeletalMesh Asset used for rendering
 * (and which is available at runtime). The FSkeletalMeshLODRenderData has vertices duplicated at any
 * split UV/normal/tangent/color, ie in the overlays there will be a unique overlay element for each
 * base mesh vertex.
 * 
 * TODO: The code mainly follows the logic of the StaticMeshLODResourcesToDynamicMesh.h with the addition of the 
 * skinning weights and bones. Both should be refactored to use ToDynamicMesh interface instead.
 */


 class FSkeletalMeshLODRenderDataToDynamicMesh
 {
 public:

	// TODO: more options?
	 struct ConversionOptions
	 {
		 bool bWantNormals = true;
		 bool bWantTangents = true;
		 bool bWantUVs = true;
		 bool bWantVertexColors = true;
		 bool bWantMaterialIDs = true;
		 bool bWantSkinWeights = true;

		 // mesh vertex positions are multiplied by the build scale
		 FVector3d BuildScale = FVector3d::One();
	 };

	// @param bHasVertexColors					Whether to add vertex colors to the output (if available)
	// @param GetVertexColorFromLODVertexIndex	Function for getting the vertex color of a given vertex index
	 static MESHCONVERSIONENGINETYPES_API bool Convert(
		 const FSkeletalMeshLODRenderData* SkeletalMeshResources,
		 const FReferenceSkeleton& RefSkeleton,
		 const ConversionOptions& Options,
		 FDynamicMesh3& OutputMesh,
		 bool bHasVertexColors,
		 TFunctionRef<FColor(int32)> GetVertexColorFromLODVertexIndex);

	 static MESHCONVERSIONENGINETYPES_API bool Convert(
		 const FSkeletalMeshLODRenderData* SkeletalMeshResources,
		 const FReferenceSkeleton& RefSkeleton,
		 const ConversionOptions& Options,
		 FDynamicMesh3& OutputMesh);
 };
}
}