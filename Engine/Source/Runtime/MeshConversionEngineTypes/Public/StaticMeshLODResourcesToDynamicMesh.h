// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

struct FStaticMeshLODResources;

namespace UE
{
namespace Geometry
{


/**
 * FStaticMeshLODResourcesToDynamicMesh can be used to create a FDynamicMesh3 from
 * a FStaticMeshLODResources, which is the variant of the mesh in a StaticMesh Asset used for rendering
 * (and which is available at runtime). The FStaticMeshLODResources has vertices duplicated at any
 * split UV/normal/tangent/color, ie in the overlays there will be a unique overlay element for each
 * base mesh vertex.
 */
class FStaticMeshLODResourcesToDynamicMesh
{
public:

	struct ConversionOptions
	{
		bool bWantNormals = true;
		bool bWantTangents = true;
		bool bWantUVs = true;
		bool bWantVertexColors = true;
		bool bWantMaterialIDs = true;

		// mesh vertex positions are multiplied by the build scale
		FVector3d BuildScale = FVector3d::One();
	};

	// @param bHasVertexColors					Whether to add vertex colors to the output (if available)
	// @param GetVertexColorFromLODVertexIndex	Function for getting the vertex color of a given vertex index
	static MESHCONVERSIONENGINETYPES_API bool Convert(
		const FStaticMeshLODResources* StaticMeshResources, 
		const ConversionOptions& Options, 
		FDynamicMesh3& OutputMesh,
		bool bHasVertexColors,
		TFunctionRef<FColor(int32)> GetVertexColorFromLODVertexIndex);

	static MESHCONVERSIONENGINETYPES_API bool Convert(
		const FStaticMeshLODResources* StaticMeshResources,
		const ConversionOptions& Options,
		FDynamicMesh3& OutputMesh);

};

}
}
